{
	"FileVersion": 3,
	"FriendlyName": "UMG Viewmodel for UMG Preview",
	"Version": 1,
	"VersionName": "1.0",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"Description": "A plugin to support UMG MVVM within the UMG Widget Preview plugin.",
	"Category": "UI",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "ModelViewViewModelPreview",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "ModelViewViewModel",
			"Enabled": true
		},
		{
			"Name": "UMGWidgetPreview",
			"Enabled": true
		}
	]
}
