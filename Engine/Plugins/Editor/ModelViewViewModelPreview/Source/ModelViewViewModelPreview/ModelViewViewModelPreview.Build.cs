// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ModelViewViewModelPreview : ModuleRules
{
	public ModelViewViewModelPreview(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.AddRange(
			new string[]
			{
				"UMGWidgetPreview",
			});

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AdvancedWidgets",
				"ApplicationCore",
				"Core",
				"CoreUObject",
				"DesktopPlatform",
				"FieldNotification",
				"Json",
				"InputCore",
				"MessageLog",
				"ModelViewViewModel",
				"ModelViewViewModelBlueprint",
				"Projects",
				"Slate",
				"SlateCore",
				"UMG",
				"UMGWidgetPreview",
			});
	}
}
