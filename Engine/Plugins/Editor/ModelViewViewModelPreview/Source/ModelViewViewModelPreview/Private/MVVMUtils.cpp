// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVMUtils.h"

#include "Containers/UnrealString.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/PlatformMemory.h"
#include "Internationalization/Text.h"
#include "Misc/Optional.h"
#include "Templates/ValueOrError.h"
#include "Types/MVVMFieldContext.h"
#include "Types/MVVMFieldVariant.h"
#include "Types/MVVMObjectVariant.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/TextProperty.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"

#define LOCTEXT_NAMESPACE "MVVMUtils"

namespace UE::MVVM::Private::Utils
{
	// Returns the FProperty for the return value of the function.
	const FProperty* GetRuntimeReturnProperty(const UFunction* InFunction)
	{
		FProperty* Result = InFunction->GetReturnProperty();

		if (Result == nullptr && InFunction->HasAllFunctionFlags(FUNC_HasOutParms))
		{
			for (TFieldIterator<FProperty> It(InFunction); It && (It->PropertyFlags & CPF_Parm); ++It)
			{
				if (It->HasAllPropertyFlags(CPF_OutParm) && !It->HasAnyPropertyFlags(CPF_Deprecated | CPF_EditorOnly | CPF_ConstParm | CPF_ReferenceParm))
				{
					Result = *It;
					break;
				}
			}
		}

		return Result;
	}

	TOptional<FText> GetPropertyValue(UObject* SourceObject, FProperty* InProperty)
	{
		if (!InProperty)
		{
			return {};
		}

		if (InProperty->IsA<FArrayProperty>())
		{
			return LOCTEXT("SourceValueArray", "[Array]");
		}
		else if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(InProperty))
		{
			if (UObject* Value = ObjectProperty->GetObjectPropertyValue_InContainer(SourceObject, 0))
			{
				return FText::FromString(Value->GetPathName());
			}
			else
			{
				return LOCTEXT("SourceValueNull", "[None]");
			}
		}
		// By default will create loctext macros.
		else if (FTextProperty* TextProperty = CastField<FTextProperty>(InProperty))
		{
			FText Value;
			TextProperty->GetValue_InContainer(SourceObject, &Value);

			return Value;
		}
		else
		{
			FString Value;
			InProperty->ExportTextItem_InContainer(Value, SourceObject, nullptr, nullptr, PPF_None);

			return FText::FromString(Value);
		}
	}

	TOptional<FText> GetFunctionValue(UObject* SourceObject, UFunction* InFunction)
	{
		if (!InFunction)
		{
			return {};
		}

		const FProperty* ReturnProperty = GetRuntimeReturnProperty(InFunction);

		if (!ReturnProperty)
		{
			return {};
		}

		// Bound functions must be const/pure and have no parameters, so only a return value
		// is necessary for the parameters "struct".

		void* DataPtr = FMemory_Alloca_Aligned(ReturnProperty->GetSize(), ReturnProperty->GetMinAlignment());
		ReturnProperty->InitializeValue(DataPtr);

		SourceObject->ProcessEvent(InFunction, DataPtr);

		FString Value;
		ReturnProperty->ExportTextItem_Direct(Value, DataPtr, nullptr, nullptr, PPF_None);

		ReturnProperty->DestroyValue(DataPtr);

		return FText::FromString(Value);
	}

	TOptional<FText> GetFieldValue(TValueOrError<UE::MVVM::FFieldContext, void>& InFieldContext, bool bInAllowFunction)
	{
		if (InFieldContext.HasError())
		{
			return {};
		}

		UObject* SourceObject = InFieldContext.GetValue().GetObjectVariant().GetUObject();

		if (!SourceObject)
		{
			return {};
		}

		if (InFieldContext.GetValue().GetFieldVariant().IsProperty())
		{
			return GetPropertyValue(SourceObject, InFieldContext.GetValue().GetFieldVariant().GetProperty());
		}
		else if (bInAllowFunction && InFieldContext.GetValue().GetFieldVariant().IsFunction())
		{
			return GetFunctionValue(SourceObject, InFieldContext.GetValue().GetFieldVariant().GetFunction());
		}

		return {};
	}
} // namespace UE::MVVM::Private::Utils

#undef LOCTEXT_NAMESPACE
