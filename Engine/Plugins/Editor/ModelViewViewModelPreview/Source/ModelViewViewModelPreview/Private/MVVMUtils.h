// Copyright Epic Games, Inc. All Rights Reserved.

class FProperty;
class FText;
class UFunction;
class UObject;
template<typename OptionalType> struct TOptional;
template<typename ValueType, typename ErrorType> class TValueOrError;

namespace UE::MVVM
{

struct FFieldContext;

namespace Private::Utils
{
	// Return a textual representation of the property's value
	// Returns an unset optional on failure.
	TOptional<FText> GetPropertyValue(UObject* SourceObject, FProperty* InProperty);

	// Call the function and return a text representation of its return value.
	// Returns an unset optional on failure.
	TOptional<FText> GetFunctionValue(UObject* SourceObject, UFunction* InFunction);

	// Return a textual representation of the field's value
	// Returns an unset optional on failure.
	TOptional<FText> GetFieldValue(TValueOrError<UE::MVVM::FFieldContext, void>& InFieldContext, bool bInAllowFunction);
} // namespace Private::Utils

} // namespace UE::MVVM