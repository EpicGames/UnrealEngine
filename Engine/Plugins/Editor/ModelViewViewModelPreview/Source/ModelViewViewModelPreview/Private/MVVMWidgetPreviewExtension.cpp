// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVMWidgetPreviewExtension.h"

#include "Framework/Docking/LayoutExtender.h"
#include "IWidgetPreviewToolkit.h"
#include "Styling/MVVMWidgetPreviewStyle.h"
#include "Widgets/SMVVMPreviewViewmodels.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "MVVMWidgetPreviewExtension"

namespace UE::MVVM::Private
{
	const FLazyName DetailsTabID(TEXT("Details"));
	const FLazyName DebugSourcePanelTabID = "WidgetPreview_ViewmodelSourcePanel";

	FName GetDetailsTabID()
	{
		return DetailsTabID.Resolve();
	}

	FName FMVVMWidgetPreviewExtension::GetPreviewSourceTabID()
	{
		return DebugSourcePanelTabID.Resolve();
	}

	void FMVVMWidgetPreviewExtension::Register(IUMGWidgetPreviewModule& InWidgetPreviewModule)
	{
		InWidgetPreviewModule.OnRegisterTabsForEditor().AddRaw(this, &FMVVMWidgetPreviewExtension::HandleRegisterPreviewEditorTab);
	}

	void FMVVMWidgetPreviewExtension::Unregister(IUMGWidgetPreviewModule* InWidgetPreviewModule)
	{
		check(InWidgetPreviewModule);

		InWidgetPreviewModule->OnRegisterTabsForEditor().RemoveAll(this);
	}

	void FMVVMWidgetPreviewExtension::HandleRegisterPreviewEditorTab(
		const TSharedPtr<UE::UMGWidgetPreview::IWidgetPreviewToolkit>& InPreviewEditor,
		const TSharedRef<FTabManager>& InTabManager)
	{
		static const FName PreviewSourceTabId = GetPreviewSourceTabID();

		const TSharedRef<FWorkspaceItem> AssetEditorTabsCategory = InTabManager->GetLocalWorkspaceMenuRoot();

		InTabManager->RegisterTabSpawner(
			PreviewSourceTabId,
			FOnSpawnTab::CreateRaw(this, &FMVVMWidgetPreviewExtension::SpawnTab_PreviewSource, TWeakPtr<UE::UMGWidgetPreview::IWidgetPreviewToolkit>(InPreviewEditor)))
			.SetDisplayName(NSLOCTEXT("DebugSourcePanel", "ViewmodelTabLabel", "Viewmodels"))
			.SetIcon(FSlateIcon(FMVVMWidgetPreviewStyle::Get().GetStyleSetName(), "BlueprintView.TabIcon"))
			.SetTooltipText(NSLOCTEXT("DebugSourcePanel", "Viewmodel_ToolTip", "Show the viewmodels panel"))
			.SetGroup(AssetEditorTabsCategory);

		if (TSharedPtr<FLayoutExtender> LayoutExtender = InPreviewEditor->GetLayoutExtender();
			LayoutExtender.IsValid())
		{
			FTabManager::FTab PreviewSourceTab(FTabId(PreviewSourceTabId, ETabIdFlags::SaveLayout), ETabState::ClosedTab);
			LayoutExtender->ExtendLayout(GetDetailsTabID(), ELayoutExtensionPosition::After, PreviewSourceTab);
		}
	}

	TSharedRef<SDockTab> FMVVMWidgetPreviewExtension::SpawnTab_PreviewSource(
		const FSpawnTabArgs& Args,
		TWeakPtr<UE::UMGWidgetPreview::IWidgetPreviewToolkit> InWeakPreviewEditor)
	{
		check(Args.GetTabId().TabType == GetPreviewSourceTabID());

		TSharedRef<SDockTab> DockTab = SNew(SDockTab);

		if (InWeakPreviewEditor.IsValid())
		{
			DockTab->SetContent(SNew(UE::MVVM::Private::SMVVMPreviewViewmodels, InWeakPreviewEditor.Pin()));
		}
		else
		{
			DockTab->SetContent(SNullWidget::NullWidget);
		}

		return DockTab;
	}
}

#undef LOCTEXT_NAMESPACE
