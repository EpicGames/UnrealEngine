// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Docking/TabManager.h"
#include "IUMGWidgetPreviewModule.h"
#include "Templates/SharedPointer.h"

namespace UE::UMGWidgetPreview
{
	class IWidgetPreviewToolkit;
}

namespace UE::MVVM::Private
{
	class FMVVMWidgetPreviewExtension
		: public TSharedFromThis<FMVVMWidgetPreviewExtension>
	{
	public:
		static FName GetPreviewSourceTabID();

		/** Register various extensibility points with the WidgetPreviewModule. */
		void Register(IUMGWidgetPreviewModule& InWidgetPreviewModule);

		void Unregister(IUMGWidgetPreviewModule* InWidgetPreviewModule);

	private:
		void HandleRegisterPreviewEditorTab(const TSharedPtr<UE::UMGWidgetPreview::IWidgetPreviewToolkit>& InPreviewEditor, const TSharedRef<FTabManager>& InTabManager);

		TSharedRef<SDockTab> SpawnTab_PreviewSource(const FSpawnTabArgs& Args, TWeakPtr<UE::UMGWidgetPreview::IWidgetPreviewToolkit> InWeakPreviewEditor);
	};
}
