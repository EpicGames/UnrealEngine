// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelViewViewModelPreviewModule.h"
#include "IUMGWidgetPreviewModule.h"
#include "MessageLogModule.h"
#include "Modules/ModuleManager.h"
#include "MVVMWidgetPreviewExtension.h"

#define LOCTEXT_NAMESPACE "ModelViewViewModelPreview"

namespace UE::UMGWidgetPreview::Private
{
	const FName WidgetPreviewModuleName = "UMGWidgetPreview";
}

const FLazyName FMVVMPreviewModule::BindingMessageLogName = "MVVMPreviewBindingMessageLog";

void FMVVMPreviewModule::StartupModule()
{
	IUMGWidgetPreviewModule& WidgetPreviewModule = FModuleManager::LoadModuleChecked<IUMGWidgetPreviewModule>(UE::UMGWidgetPreview::Private::WidgetPreviewModuleName);
	WidgetPreviewExtension = MakeShared<UE::MVVM::Private::FMVVMWidgetPreviewExtension>();
	WidgetPreviewExtension->Register(WidgetPreviewModule);

	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	FMessageLogInitializationOptions InitOptions;
	InitOptions.bShowFilters = true;
	InitOptions.bShowPages = false;
	InitOptions.bShowInLogWindow = false;
	InitOptions.bAllowClear = true;
	InitOptions.bScrollToBottom = true;
	MessageLogModule.RegisterLogListing(BindingMessageLogName, LOCTEXT("MVVMBindingLog", "MVVM Binding Log"), InitOptions);
}

void FMVVMPreviewModule::ShutdownModule()
{
	if (IUMGWidgetPreviewModule* WidgetPreviewModule = FModuleManager::GetModulePtr<IUMGWidgetPreviewModule>(UE::UMGWidgetPreview::Private::WidgetPreviewModuleName))
	{
		WidgetPreviewExtension->Unregister(WidgetPreviewModule);
	}

	if (FMessageLogModule* MessageLogModule = FModuleManager::GetModulePtr<FMessageLogModule>("MessageLog"))
	{
		MessageLogModule->UnregisterLogListing(BindingMessageLogName);
	}
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FMVVMPreviewModule, ModelViewViewModelPreview)
