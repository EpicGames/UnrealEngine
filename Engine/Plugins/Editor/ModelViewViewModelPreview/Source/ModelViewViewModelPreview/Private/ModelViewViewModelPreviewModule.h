// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

#include "Logging/LogMacros.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

namespace UE::MVVM::Private
{
	class FMVVMWidgetPreviewExtension;
}

class FMVVMPreviewModule : public IModuleInterface
{
public:
	static const FLazyName BindingMessageLogName;

	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

private:
	TSharedPtr<UE::MVVM::Private::FMVVMWidgetPreviewExtension> WidgetPreviewExtension;
};

