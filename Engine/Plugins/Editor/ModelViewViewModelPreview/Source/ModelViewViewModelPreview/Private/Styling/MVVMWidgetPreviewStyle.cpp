// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVMWidgetPreviewStyle.h"

#include "Interfaces/IPluginManager.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"

namespace UE::MVVM::Private
{
	FName FMVVMWidgetPreviewStyle::StyleName("MVVMWidgetPreview");

	FMVVMWidgetPreviewStyle& FMVVMWidgetPreviewStyle::Get()
	{
		static FMVVMWidgetPreviewStyle Instance;
		return Instance;
	}

	FMVVMWidgetPreviewStyle::FMVVMWidgetPreviewStyle()
		: FSlateStyleSet(StyleName)
	{
		const FVector2f Icon16(16.0f);
		const FVector2f Icon64(64.0f);

		TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME);
		check(Plugin.IsValid());

		SetContentRoot(Plugin->GetContentDir() / TEXT("Editor"));
		SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));

		Set("BlueprintView.TabIcon", new IMAGE_BRUSH_SVG("Slate/ViewModel", Icon16));

		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	FMVVMWidgetPreviewStyle::~FMVVMWidgetPreviewStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}
}
