// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

namespace UE::MVVM::Private
{
	class FMVVMWidgetPreviewStyle
		: public FSlateStyleSet
	{
	public:
		static FName StyleName;

		static FMVVMWidgetPreviewStyle& Get();

	private:
		FMVVMWidgetPreviewStyle();
		virtual ~FMVVMWidgetPreviewStyle() override;
	};
}
