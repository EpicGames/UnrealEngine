// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMVVMPreviewBindingLog.h"

#include "Blueprint/UserWidget.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IMessageLogListing.h"
#include "INotifyFieldValueChanged.h"
#include "Internationalization/Text.h"
#include "IWidgetPreviewToolkit.h"
#include "Logging/TokenizedMessage.h"
#include "MessageLogModule.h"
#include "Misc/Optional.h"
#include "ModelViewViewModelPreviewModule.h"
#include "Modules/ModuleManager.h"
#include "MVVMSubsystem.h"
#include "MVVMUtils.h"
#include "Types/MVVMFieldContext.h"
#include "UObject/Object.h"
#include "View/MVVMView.h"
#include "View/MVVMViewClass.h"
#include "WidgetPreview.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Views/SHeaderRow.h"

#define LOCTEXT_NAMESPACE "SMVVMPreviewBindingLog"

namespace UE::MVVM::Private
{

namespace BindingLogUtils
{
	// Returns all the value sources for a view.
	TArray<UObject*> GetViewSources(UMVVMView* InView)
	{
		if (!InView)
		{
			return {};
		}

		const TArrayView<const FMVVMView_Source> Sources = InView->GetSources();

		TArray<UObject*> NewSources;
		NewSources.Reserve(Sources.Num());

		for (const FMVVMView_Source& Source : Sources)
		{
			if (Source.Source)
			{
				NewSources.Add(Source.Source);
			}
		}

		return NewSources;
	}
	
	// Finds the first valid source based on the given source mask.
	UObject* GetFirstValidSourceByMask(const UMVVMView* InView, uint64 InSourceMask)
	{
		if (!InView)
		{
			return nullptr;
		}

		const TArrayView<const FMVVMView_Source> Sources = InView->GetSources();

		for (int32 Index = 0; Index < Sources.Num(); ++Index)
		{
			if (InSourceMask & (1ULL << Index))
			{
				if (Sources[Index].Source)
				{
					return Sources[Index].Source;
				}
			}
		}

		return nullptr;
	}

	// Find the first valid source by name
	UObject* GetFirstValidSourceByName(const UMVVMView* InView, FName InName)
	{
		if (!InView)
		{
			return nullptr;
		}

		const TArrayView<const FMVVMView_Source> Sources = InView->GetSources();

		for (int32 Index = 0; Index < Sources.Num(); ++Index)
		{
			if (Sources[Index].Source)
			{
				if (Sources[Index].Source->GetFName() == InName)
				{
					return Sources[Index].Source;
				}
			}
		}

		return nullptr;
	}
}

void SMVVMPreviewBindingLog::Construct(const FArguments& InArgs, TSharedPtr<UMGWidgetPreview::IWidgetPreviewToolkit> InPreviewEditor)
{
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	MessageLogListing = MessageLogModule.GetLogListing(FMVVMPreviewModule::BindingMessageLogName);
	MessageLogWidget = MessageLogModule.CreateLogListingWidget(MessageLogListing.ToSharedRef());

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.f, 4.f, 4.f, 4.f)
		[
			SNew(SHeaderRow)
			+ SHeaderRow::Column("Logs")
			.FillWidth(1.f)
			.DefaultLabel(LOCTEXT("BindingLog", "Binding Log"))
		]

		+ SVerticalBox::Slot()
		.FillHeight(1.f)
		.Padding(0.f, 0.f, 4.f, 4.f)
		[
			MessageLogWidget.ToSharedRef()
		]
	];

	SetPreviewEditor(InPreviewEditor);

#if UE_WITH_MVVM_DEBUGGING
	OnOnLibraryBindingExecutedHandle = FDebugging::OnLibraryBindingExecuted.AddSP(
		this,
		&SMVVMPreviewBindingLog::HandleOnLibraryBindingExecuted
	);
#endif
}

SMVVMPreviewBindingLog::~SMVVMPreviewBindingLog()
{
	if (UObjectInitialized())
	{
		SetPreviewEditor(nullptr);
	}

#if UE_WITH_MVVM_DEBUGGING
	if (OnOnLibraryBindingExecutedHandle.IsValid())
	{
		FDebugging::OnLibraryBindingExecuted.Remove(OnOnLibraryBindingExecutedHandle);
		OnOnLibraryBindingExecutedHandle.Reset();
	}
#endif
}

void SMVVMPreviewBindingLog::SetPreviewEditor(TSharedPtr<UMGWidgetPreview::IWidgetPreviewToolkit> InPreviewEditor)
{
	TSharedPtr<UMGWidgetPreview::IWidgetPreviewToolkit> CurrentPreviewEditor = WeakPreviewEditor.Pin();

	if (CurrentPreviewEditor.Get() == InPreviewEditor.Get())
	{
		return;
	}

	WeakPreviewEditor = InPreviewEditor;

	if (!InPreviewEditor.IsValid())
	{
		SetWidgetPreview(nullptr);
	}
	else
	{
		if (UWidgetPreview* WidgetPreview = InPreviewEditor->GetPreview())
		{
			SetWidgetPreview(WidgetPreview);
		}
	}
}

void SMVVMPreviewBindingLog::SetWidgetPreview(UWidgetPreview* InWidgetPreview)
{
	UWidgetPreview* CurrentWidgetPreview = WeakWidgetPreview.Get();

	if (InWidgetPreview == CurrentWidgetPreview)
	{
		return;
	}

	if (OnWidgetChangedHandle.IsValid())
	{
		if (CurrentWidgetPreview)
		{
			CurrentWidgetPreview->OnWidgetChanged().Remove(OnWidgetChangedHandle);
			OnWidgetChangedHandle.Reset();
		}
	}

	WeakWidgetPreview = InWidgetPreview;

	if (!InWidgetPreview)
	{
		SetPreviewWidget(nullptr);
	}
	else
	{
		OnWidgetChangedHandle = InWidgetPreview->OnWidgetChanged().AddSP(
			this,
			&SMVVMPreviewBindingLog::HandleOnWidgetChanged
		);

		if (UUserWidget* NewPreviewWidget = InWidgetPreview->GetWidgetInstance())
		{
			SetPreviewWidget(NewPreviewWidget);
		}
	}
}

void SMVVMPreviewBindingLog::SetPreviewWidget(UUserWidget* InWidget)
{
	if (UMVVMView* CurrentView = WeakView.Get())
	{
		UUserWidget* CurrentWidget = Cast<UUserWidget>(CurrentView->GetOuter());

		if (CurrentWidget == InWidget)
		{
			return;
		}
	}

	// Don't store a weak reference to the user widget as we're not binding anything to it.

	if (!InWidget)
	{
		SetView(nullptr);
	}
	else
	{
		SetView(UMVVMSubsystem::GetViewFromUserWidget(InWidget));
	}	
}

void SMVVMPreviewBindingLog::SetView(UMVVMView* InView)
{
	UMVVMView* CurrentView = WeakView.Get();

	if (InView == CurrentView)
	{
		return;
	}

#if UE_WITH_MVVM_DEBUGGING
	if (OnViewSourceChangedHandle.IsValid())
	{
		FDebugging::OnViewSourceValueChanged.Remove(OnViewSourceChangedHandle);
		OnViewSourceChangedHandle.Reset();
	}
#endif

	WeakView = InView;

	if (!InView)
	{
		SetSources({});
	}
	else
	{
#if UE_WITH_MVVM_DEBUGGING
		OnViewSourceChangedHandle = FDebugging::OnViewSourceValueChanged.AddSP(
			this,
			&SMVVMPreviewBindingLog::HandleOnViewSourceChanged
		);
#endif	

		SetSources(BindingLogUtils::GetViewSources(InView));
	}
}

void SMVVMPreviewBindingLog::SetSources(const TArray<UObject*>& InSources)
{
	for (const TWeakObjectPtr<UObject>& WeakSource : WeakSources)
	{
		UObject* Source = WeakSource.Get();

		if (!Source)
		{
			continue;
		}

		INotifyFieldValueChanged* NotifyField = Cast<INotifyFieldValueChanged>(Source);

		if (!NotifyField)
		{
			continue;
		}

		NotifyField->RemoveAllFieldValueChangedDelegates(this);
	}

	WeakSources.Empty(InSources.Num());

	for (UObject* Source : InSources)
	{
		INotifyFieldValueChanged* NotifyField = Cast<INotifyFieldValueChanged>(Source);

		if (!NotifyField)
		{
			continue;
		}

		const UE::FieldNotification::IClassDescriptor& ClassDescriptor = NotifyField->GetFieldNotificationDescriptor();

		ClassDescriptor.ForEachField(
			Source->GetClass(),
			[this, NotifyField](UE::FieldNotification::FFieldId InFieldId)
			{
				NotifyField->AddFieldValueChangedDelegate(
					InFieldId,
					INotifyFieldValueChanged::FFieldValueChangedDelegate::CreateSP(
						this,
						&SMVVMPreviewBindingLog::HandleOnFieldValueChanged
					)
				);

				return true;
			}
		);

		WeakSources.Add(Source);
	}
}


void SMVVMPreviewBindingLog::HandleOnWidgetChanged(const EWidgetPreviewWidgetChangeType InChangeType)
{
	// Make sure that we still have the correct preview widget
	if (UWidgetPreview* WidgetPreview = WeakWidgetPreview.Get())
	{
		SetPreviewWidget(WidgetPreview->GetWidgetInstance());
	}
	else
	{
		SetPreviewWidget(nullptr);
	}
}

#if UE_WITH_MVVM_DEBUGGING
void SMVVMPreviewBindingLog::HandleOnViewSourceChanged(const FDebugging::FView& InView, const FDebugging::FViewSourceValueArgs& InArgs)
{
	SetSources(BindingLogUtils::GetViewSources(WeakView.Get()));
}
#endif

void SMVVMPreviewBindingLog::HandleOnFieldValueChanged(UObject* InSourceObject, UE::FieldNotification::FFieldId InFieldId)
{
	if (!InSourceObject)
	{
		return;
	}

	UE::FieldNotification::FFieldVariant Field = InFieldId.ToVariant(InSourceObject);

	if (!Field.IsValid())
	{
		return;
	}

	TSharedRef<FTokenizedMessage> SourceLog = MakeLog(
		EMessageSeverity::Info,
		LOCTEXT("SourceLog", "Trigger: ")
	);

	if (UWidget* Widget = Cast<UWidget>(InSourceObject))
	{
		AddSourceToken(SourceLog, Widget);
	}
	else
	{
		AddSourceToken(SourceLog, InSourceObject);
	}

	if (Field.IsProperty())
	{
		SourceLog->AddText(INVTEXT("."));
		SourceLog->AddText(Field.GetProperty()->GetDisplayNameText());
	}
	else if (Field.IsFunction())
	{
		SourceLog->AddText(INVTEXT("."));
		SourceLog->AddText(Field.GetFunction()->GetDisplayNameText());
		SourceLog->AddText(INVTEXT("()"));
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("Invalid field type not caught by FieldVariant.IsValid()"));
		SourceLog->AddText(INVTEXT(".???"));
		SourceLog->SetSeverity(EMessageSeverity::Error);
	}

	using namespace BindingLogUtils;

	TSharedRef<FTokenizedMessage> SourceValueLog = MakeLog(
		EMessageSeverity::Info,
		LOCTEXT("SourceValueLog", "Source Value: ")
	);

	SourceValueLog->SetIndentationLevel(2);

	bool bHasError = false;

	if (FProperty* Property = Field.GetProperty())
	{
		if (TOptional<FText> Value = Utils::GetPropertyValue(InSourceObject, Property))
		{
			AddCopyToken(SourceValueLog, Value.GetValue());
		}
		else
		{
			SourceValueLog->AddText<FText>(LOCTEXT("SourceValueError_PropValue", "Error Failed to get value for property %s"), FText::FromString(Property->GetName()));
			bHasError = true;
		}
	}
	else if (UFunction* Function = Field.GetFunction())
	{
		if (TOptional<FText> Value = Utils::GetFunctionValue(InSourceObject, Function))
		{
			AddCopyToken(SourceValueLog, Value.GetValue());
		}
		else
		{
			SourceValueLog->AddText<FText>(LOCTEXT("SourceValueError_FuncValue", "Error Failed to get value for function %s"), FText::FromString(Function->GetName()));
			bHasError = true;
		}
	}
	else
	{
		SourceValueLog->AddText<FText>(LOCTEXT("SourceValueError_FieldValue", "Error Failed to get value for field %s"), FText::FromName(Field.GetFName()));
		bHasError = true;
	}
}


#if UE_WITH_MVVM_DEBUGGING
void SMVVMPreviewBindingLog::HandleOnLibraryBindingExecuted(const FDebugging::FView& InView, const FDebugging::FLibraryBindingExecutedArgs& InArgs)
{
	const UMVVMView* View = InView.GetView();

	if (!View || View != WeakView.Get())
	{
		return;
	}

	const UMVVMViewClass* ViewClass = View->GetViewClass();
	
	if (!ViewClass)
	{
		return;
	}

	const TArrayView<const FMVVMViewClass_Binding> Bindings = ViewClass->GetBindings();

	if (!Bindings.IsValidIndex(InArgs.Binding.GetIndex()))
	{
		return;
	}

	const FMVVMVCompiledBinding& Binding = Bindings[InArgs.Binding.GetIndex()].GetBinding();

	UUserWidget* UserWidget = Cast<UUserWidget>(View->GetOuter());

	/*
	 * Add binding destination
	 */
	TValueOrError<FString, FString> DestinationFieldPathString = ViewClass->GetBindingLibrary().FieldPathToString(Binding.GetDestinationFieldPath(), /* Display name */ true);
	FString DestinationString = DestinationFieldPathString.HasValue() ? DestinationFieldPathString.StealValue() : DestinationFieldPathString.StealError();

	TSharedRef<FTokenizedMessage> DestinationLog = MakeLog(
		(DestinationFieldPathString.HasError() || DestinationString.IsEmpty())
			? EMessageSeverity::Error
			: EMessageSeverity::Info,
		LOCTEXT("DestinationLog", "Set:")
	);

	DestinationLog->SetIndentationLevel(1);

	bool bHasError = false;
	FString DestinationName;

	if (DestinationFieldPathString.HasError())
	{
		const FText ErrorFormat = LOCTEXT("DestinationErrorFormat", "(Error: {0})");
		DestinationName += FText::Format(ErrorFormat, FText::FromString(DestinationString)).ToString();
	}
	else if (DestinationString.IsEmpty())
	{
		DestinationName += LOCTEXT("DestinationMissing", "???").ToString();
	}
	else
	{
		if (UserWidget)
		{
			AddSourceToken(DestinationLog, UserWidget);
			DestinationName = TEXT(".");
		}

		DestinationName += DestinationString;
	}

	DestinationLog->AddToken(FTextToken::Create(FText::FromString(DestinationName)));

	/*
	 * Add destination value if there's a conversion function
	 * The value may/will differ from the source value. Explicitly add a message about the transformed value.
	 */
	if (UserWidget && Binding.GetDestinationFieldPath().IsValid() && Binding.GetConversionFunctionFieldPath().IsValid())
	{
		TValueOrError<UE::MVVM::FFieldContext, void> DestinationFieldContext = ViewClass->GetBindingLibrary().EvaluateFieldPath(UserWidget, Binding.GetDestinationFieldPath());

		TSharedRef<FTokenizedMessage> DestinationValueLog = MakeLog(
			DestinationFieldPathString.HasError()
				? EMessageSeverity::Error
				: EMessageSeverity::Info,
			LOCTEXT("DestinationValueLog", "Target Value: ")
		);

		DestinationValueLog->SetIndentationLevel(3);

		/*
		 * Prepend conversion information to the value
		 */
		if (Binding.GetConversionFunctionFieldPath().IsValid())
		{
			FText FormattedConversion;

			TValueOrError<FString, FString> ConversionFieldPathString = ViewClass->GetBindingLibrary().FieldPathToString(Binding.GetConversionFunctionFieldPath(), false);
			const FString ConversionString = ConversionFieldPathString.HasValue() ? ConversionFieldPathString.StealValue() : ConversionFieldPathString.StealError();

			if (!ConversionString.IsEmpty())
			{
				if (ConversionFieldPathString.HasError())
				{
					const FText ConversionErrorFormat = LOCTEXT("ConversionErrorFormat", "(Error: {0})");
					FormattedConversion = FText::Format(ConversionErrorFormat, FText::FromString(ConversionString));
				}
				else
				{
					const FText ConversionErrorFormat = LOCTEXT("ConversionLogFormat", "({0})");
					FormattedConversion = FText::Format(ConversionErrorFormat, FText::FromString(ConversionString));
				}
			}

			if (!FormattedConversion.IsEmpty())
			{
				DestinationValueLog->AddToken(FTextToken::Create(FormattedConversion));
				DestinationValueLog->AddToken(FTextToken::Create(INVTEXT(" ")));
			}
		}

		if (TOptional<FText> Value = Utils::GetFieldValue(DestinationFieldContext, /* Allow Function */ false))
		{
			AddCopyToken(DestinationValueLog, Value.GetValue());
		}
		else
		{
			DestinationValueLog->AddText(LOCTEXT("DestinationValueError", "Error"));
		};
	}
}
#endif

TSharedRef<FTokenizedMessage> SMVVMPreviewBindingLog::MakeLog(EMessageSeverity::Type InSeverity, const FText& InMessage)
{
	TSharedRef<FTokenizedMessage> NewMessage = FTokenizedMessage::Create(InSeverity);
	NewMessage->AddToken(FTextToken::Create(InMessage));
	MessageLogListing->AddMessage(NewMessage);
	return NewMessage;
}

void SMVVMPreviewBindingLog::HandleObjectTokenClicked(TWeakObjectPtr<UObject> InWeakObject)
{
	if (TSharedPtr<UMGWidgetPreview::IWidgetPreviewToolkit> PreviewEditor = WeakPreviewEditor.Pin())
	{
		if (UObject* ClickedObject = InWeakObject.Get())
		{
			PreviewEditor->SetSelectedObjects({ClickedObject});
		}
	}
}

void SMVVMPreviewBindingLog::AddSourceToken(TSharedRef<FTokenizedMessage> InMessage, TNotNull<UObject*> InSource)
{
	if (UMVVMView* View = WeakView.Get())
	{
		const UObject* ViewOuter = View->GetOuter();

		const FText Path = (InSource == ViewOuter)
			? FText::FromName(InSource->GetFName())
			: FText::FromString(InSource->GetPathName(View->GetOuter()));

		InMessage->AddToken(
			FActionToken::Create(
				Path,
				LOCTEXT("SelectSource", "Select Source"),
				FOnActionTokenExecuted::CreateSP(
					this,
					&SMVVMPreviewBindingLog::HandleObjectTokenClicked,
					TWeakObjectPtr<UObject>(InSource)
				)
			)
		);
	}
}

void SMVVMPreviewBindingLog::AddCopyToken(TSharedRef<FTokenizedMessage> InMessage, const FText& InText)
{
	InMessage->AddText(InText);

	InMessage->AddText(INVTEXT(" "));

	InMessage->AddToken(
		FActionToken::Create(
			LOCTEXT("CopyLink", "Copy"),
			LOCTEXT("CopyText", "Copy to the Clipboard"),
			FOnActionTokenExecuted::CreateSP(
				this,
				&SMVVMPreviewBindingLog::HandleCopyTokenClicked,
				InText
			)
		)
	);
}

void SMVVMPreviewBindingLog::HandleCopyTokenClicked(FText InText)
{
	FPlatformApplicationMisc::ClipboardCopy(*InText.ToString());
}

}

#undef LOCTEXT_NAMESPACE

