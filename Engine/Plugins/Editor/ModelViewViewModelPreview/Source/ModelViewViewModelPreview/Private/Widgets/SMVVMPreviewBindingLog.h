// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "Delegates/IDelegateInstance.h"
#include "UObject/WeakObjectPtr.h"

#if UE_WITH_MVVM_DEBUGGING
#include "Debugging/MVVMDebugging.h"
#endif

class FText;
class FTokenizedMessage;
class IMessageLogListing;
class INotifyFieldValueChanged;
class UMVVMView;
class UObject;
class UUserWidget;
class UWidgetPreview;
enum class EWidgetPreviewWidgetChangeType : uint8;

namespace EMessageSeverity
{
	enum Type : int;
}

namespace UE::UMGWidgetPreview
{
	class IWidgetPreviewToolkit;
}

namespace UE::FieldNotification
{
	struct FFieldId;
}

namespace UE::MVVM::Private
{

class SPreviewSourceView;
class SPreviewSourceEntry;

/** Log to display the changes in the UMG Preview Widget MVVM Values/Bindings */
class SMVVMPreviewBindingLog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMVVMPreviewBindingLog) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<UMGWidgetPreview::IWidgetPreviewToolkit> InPreviewEditor);

	virtual ~SMVVMPreviewBindingLog() override;

private:
	TWeakPtr<UMGWidgetPreview::IWidgetPreviewToolkit> WeakPreviewEditor;
	TWeakObjectPtr<UWidgetPreview> WeakWidgetPreview;
	TWeakObjectPtr<UMVVMView> WeakView;

	FDelegateHandle OnWidgetChangedHandle;

#if UE_WITH_MVVM_DEBUGGING
	FDelegateHandle OnOnLibraryBindingExecutedHandle;
#endif

	// Must implement INotifyFieldValueChanged
	// Stored here to remember whose events we subscribed to.
	TArray<TWeakObjectPtr<UObject>> WeakSources;

#if UE_WITH_MVVM_DEBUGGING
	FDelegateHandle OnViewSourceChangedHandle;
#endif

	TSharedPtr<IMessageLogListing> MessageLogListing;
	TSharedPtr<SWidget> MessageLogWidget;

	// Sets and unsets the preview editor, adding and removing bindings.
	void SetPreviewEditor(TSharedPtr<UMGWidgetPreview::IWidgetPreviewToolkit> InPreviewEditor);

	// Sets and unsets the widget preview object, adding and removing bindings.
	void SetWidgetPreview(UWidgetPreview* InWidgetPreview);

	// Checks the current view to see if we need to update it with this widget's view.
	void SetPreviewWidget(UUserWidget* InWidget);

	// Sets and unsets the view.
	void SetView(UMVVMView* InView);

	// When the widget in the preview is changed, update the view.
	void HandleOnWidgetChanged(const EWidgetPreviewWidgetChangeType InChangeType);

#if UE_WITH_MVVM_DEBUGGING
	void HandleOnLibraryBindingExecuted(const FDebugging::FView& InView, const FDebugging::FLibraryBindingExecutedArgs& InArgs);
#endif

	// Removes previous binds to field notify sources and adds new binds to the provided sources.
	void SetSources(const TArray<UObject*>& InSources);

#if UE_WITH_MVVM_DEBUGGING
	// When the sources change, update the source bindings.
	void HandleOnViewSourceChanged(const FDebugging::FView& InView, const FDebugging::FViewSourceValueArgs& InArgs);
#endif

	// When a field notify value changes, produce a log entry.
	void HandleOnFieldValueChanged(UObject* InSourceObject, UE::FieldNotification::FFieldId InFieldId);

	// Creates a log message in the binding log message log and returns it.
	TSharedRef<FTokenizedMessage> MakeLog(EMessageSeverity::Type InSeverity, const FText& InMessage);

	// Adds an action token that selects the given source object.
	void AddSourceToken(TSharedRef<FTokenizedMessage> InMessage, TNotNull<UObject*> InSource);
		
	// Callback to select an object.
	void HandleObjectTokenClicked(TWeakObjectPtr<UObject> InWeakObject);

	// Adds an action that copies the given text to the clipboard
	void AddCopyToken(TSharedRef<FTokenizedMessage> InMessage, const FText& InText);

	// Handles the copy callback to copy text to the clipboard
	void HandleCopyTokenClicked(FText InText);
};

}
