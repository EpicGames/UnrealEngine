// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMVVMPreviewSourcePanel.h"

#include "Blueprint/UserWidget.h"
#include "DesktopPlatformModule.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Framework/Application/SlateApplication.h"
#include "IDesktopPlatform.h"
#include "IFieldNotificationClassDescriptor.h"
#include "IWidgetPreviewToolkit.h"
#include "Logging/LogMacros.h"
#include "Misc/FileHelper.h"
#include "Misc/MessageDialog.h"
#include "MVVMMessageLog.h"
#include "MVVMSubsystem.h"
#include "MVVMUtils.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "Styling/AppStyle.h"
#include "UObject/Object.h"
#include "View/MVVMView.h"
#include "View/MVVMViewClass.h"
#include "WidgetPreview.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/PropertyViewer/SFieldIcon.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SMVVMDebugSourcePanel"

namespace UE::MVVM::Private
{
	class SPreviewSourceView final : public SListView<TSharedPtr<SPreviewSourceEntry>>
	{
	};

	class SPreviewSourceEntry
	{
	public:
		SPreviewSourceEntry(UObject* InInstance, FName InName)
			: WeakInstance(InInstance)
			, Name(InName)
		{}

		UClass* GetClass() const
		{
			UObject* Instance = WeakInstance.Get();
			return Instance ? Instance->GetClass() : nullptr;
		}

		FText GetDisplayName() const
		{
			return FText::FromName(Name);
		}

		UObject* GetInstance() const
		{
			return WeakInstance.Get();
		}

	private:
		TWeakObjectPtr<UObject> WeakInstance;
		FName Name;
	};

	void SPreviewSourcePanel::Construct(const FArguments& InArgs, TSharedPtr<UMGWidgetPreview::IWidgetPreviewToolkit> PreviewEditor)
	{
		WeakPreviewEditor = PreviewEditor;
		check(PreviewEditor);

		if (UWidgetPreview* Preview = PreviewEditor->GetPreview())
		{
			HandlePreviewWidgetChanged(EWidgetPreviewWidgetChangeType::Assignment);
			OnWidgetChangedHandle = Preview->OnWidgetChanged().AddSP(this, &SPreviewSourcePanel::HandlePreviewWidgetChanged);
	#if UE_WITH_MVVM_DEBUGGING
			FDebugging::OnViewSourceValueChanged.AddSP(this, &SPreviewSourcePanel::HandleViewChanged);
	#endif
		}

		OnSelectedObjectsChangedHandle = PreviewEditor->OnSelectedObjectsChanged().AddSP(this, &SPreviewSourcePanel::HandleSelectedObjectChanged);

		ChildSlot
		[
			SNew(SBorder)
			.BorderImage(&FAppStyle::Get().GetWidgetStyle<FTableViewStyle>("ListView").BackgroundBrush)
			.Padding(4.f)
			[
				SAssignNew(SourceListView, Private::SPreviewSourceView)
				.ListItemsSource(&SourceList)
				.SelectionMode(ESelectionMode::Single)
				.OnGenerateRow(this, &SPreviewSourcePanel::GenerateWidget)
				.OnSelectionChanged(this, &SPreviewSourcePanel::HandleSourceSelectionChanged)
				.OnContextMenuOpening(this, &SPreviewSourcePanel::OnContextMenuOpened)
				.HeaderRow(
					SNew(SHeaderRow)
					+ SHeaderRow::Column("Sources")
					.FillWidth(1.f)
					.DefaultLabel(LOCTEXT("Sources", "Sources"))
				)
			]
		];

		HandlePreviewWidgetChanged(EWidgetPreviewWidgetChangeType::Reinstanced);
	}

	SPreviewSourcePanel::~SPreviewSourcePanel()
	{
		if (TSharedPtr<UMGWidgetPreview::IWidgetPreviewToolkit> PreviewEditor = WeakPreviewEditor.Pin())
		{
			if (UWidgetPreview* Preview = PreviewEditor->GetPreview())
			{
				Preview->OnWidgetChanged().Remove(OnWidgetChangedHandle);
			}

			PreviewEditor->OnSelectedObjectsChanged().Remove(OnSelectedObjectsChangedHandle);
		}
	}

	void SPreviewSourcePanel::HandlePreviewWidgetChanged(const EWidgetPreviewWidgetChangeType InChangeType)
	{
		SourceList.Reset();
		WeakView.Reset();

		if (TSharedPtr<UMGWidgetPreview::IWidgetPreviewToolkit> PreviewEditor = WeakPreviewEditor.Pin())
		{
			if (UUserWidget* NewWidget = PreviewEditor->GetPreview() ? PreviewEditor->GetPreview()->GetWidgetInstance() : nullptr)
			{
				if (UMVVMView* View = UMVVMSubsystem::GetViewFromUserWidget(NewWidget))
				{
					WeakView = View;

					const bool bShouldDefaultConstructViewmodels = PreviewEditor->GetPreview()->ShouldDefaultConstructViewmodels();

					for (const FMVVMView_Source& ViewSource : View->GetSources())
					{
						/**
						 * If we don't have a viewmodel, create one now. 
						 * If the Viewmodel creation mode is manual, global or resolver based the viewmodel won't exist in editor
						 * but we still want to show something in the preview.
						 */
						if (bShouldDefaultConstructViewmodels && !ViewSource.Source)
						{
							const FMVVMViewClass_Source& ClassSource = View->GetViewClass()->GetSource(ViewSource.ClassKey);
							
							if (UClass* SourceClass = ClassSource.GetSourceClass())
							{
								if (UObject* NewSource = NewObject<UObject>(NewWidget, SourceClass, NAME_None, RF_Transient))
								{
									View->SetViewModelByClass(NewSource);
								}
							}
						}

						FName SourceName = View->GetViewClass()->GetSource(ViewSource.ClassKey).GetName();
						SourceList.Emplace(MakeShared<Private::SPreviewSourceEntry>(ViewSource.Source, SourceName));
					}
				}
			}
		}

		if (SourceListView)
		{
			SourceListView->RequestListRefresh();
		}
	}

	void SPreviewSourcePanel::HandleSelectedObjectChanged(const TConstArrayView<TWeakObjectPtr<UObject>> InSelectedObjects)
	{
		if (bIsSelectingListItem)
		{
			return;
		}

		TGuardValue<bool> TmpInternalSelection(bIsSelectingListItem, true);
		if (TSharedPtr<UMGWidgetPreview::IWidgetPreviewToolkit> PreviewEditor = WeakPreviewEditor.Pin())
		{
			if (InSelectedObjects.Num() == 1)
			{
				TSharedPtr<Private::SPreviewSourceEntry>* Found = SourceList.FindByPredicate(
					[ToFind = InSelectedObjects[0]](const TSharedPtr<Private::SPreviewSourceEntry>& Other)
					{
						return Other->GetInstance() == ToFind;
					});

				if (Found)
				{
					SourceListView->SetSelection(*Found);
				}
				else
				{
					SourceListView->SetSelection(nullptr);
				}
			}
			else
			{
				SourceListView->SetSelection(nullptr);
			}
		}
	}

	void SPreviewSourcePanel::HandleSourceSelectionChanged(TSharedPtr<Private::SPreviewSourceEntry> Entry, ESelectInfo::Type SelectionType) const
	{
		if (bIsSelectingListItem)
		{
			return;
		}

		TGuardValue<bool> TmpInternalSelection(bIsSelectingListItem, true);
		if (TSharedPtr<UMGWidgetPreview::IWidgetPreviewToolkit> PreviewEditor = WeakPreviewEditor.Pin())
		{
			if (Entry && Entry->GetInstance())
			{
				PreviewEditor->SetSelectedObjects({ Entry->GetInstance() });
			}
			else // Nothing selected
			{
				// Effectively clears the selection
				PreviewEditor->SetSelectedObjects({ });
			}
		}
	}

	#if UE_WITH_MVVM_DEBUGGING
	void SPreviewSourcePanel::HandleViewChanged(const FDebugging::FView& View, const FDebugging::FViewSourceValueArgs& Args)
	{
		if (SourceListView)
		{
			if (View.GetView() == WeakView.Get())
			{
				SourceListView->RebuildList(); // to prevent access to invalid class, rebuild everything.
			}
		}
	}
	#endif

	TSharedRef<ITableRow> SPreviewSourcePanel::GenerateWidget(TSharedPtr<Private::SPreviewSourceEntry> Entry, const TSharedRef<STableViewBase>& OwnerTable) const
	{
		using RowType = STableRow<TSharedPtr<Private::SPreviewSourceEntry>>;

		const TSharedRef<SWidget> FieldIcon = Entry->GetClass() ? SNew(UE::PropertyViewer::SFieldIcon, Entry->GetClass()) : SNullWidget::NullWidget;

		TSharedRef<RowType> NewRow = SNew(RowType, OwnerTable);

		NewRow->SetContent(SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				FieldIcon
			]
			+ SHorizontalBox::Slot()
			.Padding(4.0f)
			[
				SNew(STextBlock)
				.Text(Entry->GetDisplayName())
			]);

		return NewRow;
	}

	TSharedPtr<SWidget> SPreviewSourcePanel::OnContextMenuOpened()
	{
		if (SourceListView.IsValid())
		{
			TArray<TSharedPtr<Private::SPreviewSourceEntry>> SelectedItems = SourceListView->GetSelectedItems();

			if (SelectedItems.Num() == 1)
			{
				constexpr bool bInShouldCloseWindowAfterMenuSelection = true;
				FMenuBuilder MenuBuilder(bInShouldCloseWindowAfterMenuSelection, nullptr);

				MenuBuilder.BeginSection(TEXT("Viewmodel"), LOCTEXT("Viewmodel", "Viewmodel"));
				{
					FMenuEntryParams Params;
					Params.LabelOverride = LOCTEXT("ViewmodelDataDump", "Dump Data to JSON");
					Params.UserInterfaceActionType = EUserInterfaceActionType::Button;
					Params.DirectActions.ExecuteAction.BindSP(
						this,
						&SPreviewSourcePanel::ExecuteDumpData,
						SelectedItems[0]
					);
					
					MenuBuilder.AddMenuEntry(Params);
				}
				MenuBuilder.EndSection();

				return MenuBuilder.MakeWidget();
			}
		}

		return SNullWidget::NullWidget;
	}

	void SPreviewSourcePanel::ExecuteDumpData(TSharedPtr<Private::SPreviewSourceEntry> InEntry)
	{
		if (!InEntry.IsValid())
		{
			return;
		}
		
		UObject* Entry = InEntry->GetInstance();

		if (!Entry)
		{
			return;
		}

		if (!Entry->Implements<UNotifyFieldValueChanged>())
		{
			UE_LOGF(LogMVVM, Error, "Object selected for dump does not implement INotifyFieldValueChanged. [%ls]", *Entry->GetPathName());
			return;
		}

		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::TryGet();

		if (!DesktopPlatform)
		{
			UE_LOGF(LogMVVM, Error, "Unable to retrieve desktop platform instance.");
			return;
		}

		static const TArray<TPair<TCHAR, TCHAR>> ValidCharacterRanges = {
			{'A', 'Z'},
			{'a', 'z'},
			{'0', '9'},
			{' ', ' '},
			{'-', '-'},
			{'_', '_'}
		};

		FString Filename = Entry->GetPathName();

		for (int32 Index = 0; Index < Filename.Len(); ++Index)
		{
			bool bIsValidChar = false;

			for (const TPair<TCHAR, TCHAR>& CharRange : ValidCharacterRanges)
			{
				if (Filename[Index] >= CharRange.Key && Filename[Index] <= CharRange.Value)
				{
					bIsValidChar = true;
					break;
				}
			}

			if (!bIsValidChar)
			{
				Filename[Index] = TEXT('_');
			}
		}

		TArray<FString> Filenames;
		Filenames.Reserve(1);

		const bool bSuccess = DesktopPlatform->SaveFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(SharedThis(this)),
			LOCTEXT("DumpDataTitle", "Export Viewmodel to...").ToString(),
			FPaths::ProjectSavedDir(),
			Filename,
			TEXT("json (*.json)|*.json|"),
			EFileDialogFlags::None,
			Filenames
		);

		// Nothing picked
		if (!bSuccess || Filenames.IsEmpty())
		{
			return;
		}

		TSharedRef<FJsonObject> JsonRoot = MakeShared<FJsonObject>();
		TSharedRef<FJsonObject> Functions = MakeShared<FJsonObject>();
		TSharedRef<FJsonObject> Properties = MakeShared<FJsonObject>();
		
		JsonRoot->SetField(TEXT("Functions"), MakeShared<FJsonValueObject>(Functions));
		JsonRoot->SetField(TEXT("Properties"), MakeShared<FJsonValueObject>(Properties));

		INotifyFieldValueChanged* FieldNotifier = Cast<INotifyFieldValueChanged>(Entry);
		const UE::FieldNotification::IClassDescriptor& ClassDescriptor = FieldNotifier->GetFieldNotificationDescriptor();
		
		ClassDescriptor.ForEachField(
			Entry->GetClass(),
			[this, Entry, Functions, Properties](UE::FieldNotification::FFieldId InFieldId) -> bool
			{
				UE::FieldNotification::FFieldVariant Field = InFieldId.ToVariant(Entry);

				if (UFunction* Function = Field.GetFunction())
				{
					Functions->SetStringField(
						Function->GetName(),
						Utils::GetFunctionValue(Entry, Function).Get(FText::GetEmpty()).ToString()
					);
				}
				else if (FProperty* Property = Field.GetProperty())
				{
					Properties->SetStringField(
						Property->GetName(),
						Utils::GetPropertyValue(Entry, Property).Get(FText::GetEmpty()).ToString()
					);
				}

				return true;
			}
		);

		FString SerializedString;
		const TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&SerializedString);
		FJsonSerializer::Serialize(JsonRoot, Writer);

		if (FFileHelper::SaveStringToFile(SerializedString, *Filenames[0]))
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("DumpSuccess", "Viewmodel data dumped to JSON."));
		}
		else
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("DumpFailure", "Failed to dump Viewmodel data to JSON."));
		}
	}
}

#undef LOCTEXT_NAMESPACE
