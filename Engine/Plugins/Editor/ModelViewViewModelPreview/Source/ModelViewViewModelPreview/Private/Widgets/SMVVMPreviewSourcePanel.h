// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Debugging/MVVMDebugging.h"
#include "WidgetPreview.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::UMGWidgetPreview
{
	class IWidgetPreviewToolkit;
}

class STableViewBase;
class ITableRow;
class UMVVMView;
class UUserWidget;

#ifndef UE_WITH_MVVM_DEBUGGING
  #error "MVVM Debugging needs to be enabled."
#endif

namespace UE::MVVM::Private { class SPreviewSourceView; }
namespace UE::MVVM::Private { class SPreviewSourceEntry; }

namespace UE::MVVM::Private
{
	/** */
	class SPreviewSourcePanel : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SPreviewSourcePanel) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedPtr<UMGWidgetPreview::IWidgetPreviewToolkit> PreviewEditor);

		virtual ~SPreviewSourcePanel() override;

	private:
		void HandlePreviewWidgetChanged(const EWidgetPreviewWidgetChangeType InChangeType);
		void HandleSelectedObjectChanged(const TConstArrayView<TWeakObjectPtr<UObject>> InSelectedObjects);
		void HandleSourceSelectionChanged(TSharedPtr<Private::SPreviewSourceEntry> Entry, ESelectInfo::Type SelectionType) const;
	#if UE_WITH_MVVM_DEBUGGING
		void HandleViewChanged(const FDebugging::FView&, const FDebugging::FViewSourceValueArgs&);
	#endif
		TSharedRef<ITableRow> GenerateWidget(TSharedPtr<Private::SPreviewSourceEntry> Entry, const TSharedRef<STableViewBase>& OwnerTable) const;
		TSharedPtr<SWidget> OnContextMenuOpened();
		void ExecuteDumpData(TSharedPtr<Private::SPreviewSourceEntry> InEntry);

		TWeakObjectPtr<UMVVMView> WeakView;
		TWeakPtr<UMGWidgetPreview::IWidgetPreviewToolkit> WeakPreviewEditor;
		TSharedPtr<Private::SPreviewSourceView> SourceListView;
		TArray<TSharedPtr<Private::SPreviewSourceEntry>> SourceList;
		mutable bool bIsSelectingListItem = false;

		FDelegateHandle OnWidgetChangedHandle;
		FDelegateHandle OnSelectedObjectsChangedHandle;
	};
}

