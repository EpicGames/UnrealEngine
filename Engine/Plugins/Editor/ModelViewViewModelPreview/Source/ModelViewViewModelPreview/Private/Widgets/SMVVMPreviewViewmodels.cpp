// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMVVMPreviewViewmodels.h"

#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SMVVMPreviewBindingLog.h"
#include "Widgets/SMVVMPreviewSourcePanel.h"

namespace UE::MVVM::Private
{

void SMVVMPreviewViewmodels::Construct(const FArguments& InArgs, TSharedPtr<UMGWidgetPreview::IWidgetPreviewToolkit> InPreviewEditor)
{
	SourcePanel = SNew(SPreviewSourcePanel, InPreviewEditor)
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("PreviewSourcePanel")));

	BindingLog = SNew(SMVVMPreviewBindingLog, InPreviewEditor)
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("PreviewBindingLog")));

	ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Fill)
		[
			SourcePanel.ToSharedRef()
		]

		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		.VAlign(VAlign_Fill)
		[
			BindingLog.ToSharedRef()
		]
	];
}

}
