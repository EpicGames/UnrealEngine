// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

namespace UE::UMGWidgetPreview
{
	class IWidgetPreviewToolkit;
}

namespace UE::MVVM::Private
{

class SPreviewSourceView;
class SPreviewSourceEntry;

/** */
class SMVVMPreviewViewmodels : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMVVMPreviewViewmodels) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<UMGWidgetPreview::IWidgetPreviewToolkit> InPreviewEditor);

private:
	TSharedPtr<SWidget> SourcePanel;
	TSharedPtr<SWidget> BindingLog;
};

}
